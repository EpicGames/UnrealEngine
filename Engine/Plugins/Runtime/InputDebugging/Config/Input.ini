[/Script/EngineSettings.ConsoleSettings]
+ManualAutoCompleteList=(Command="ShowDebug DeviceProperty",Desc="Display debug information about currently active Input Device Properties")
+ManualAutoCompleteList=(Command="ShowDebug Devices",Desc="Displays all current Platform Users and their Input Devices.")