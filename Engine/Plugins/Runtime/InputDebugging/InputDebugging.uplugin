{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Input Debugging",
	"Description": "Input debugging and visualization.",
	"Category": "Input",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "InputDebugging",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetConfigurationDenyList": [ "Shipping" ]
		},
		{
			"Name": "InputDebuggingEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}