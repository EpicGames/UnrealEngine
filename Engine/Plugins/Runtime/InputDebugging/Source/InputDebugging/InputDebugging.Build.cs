// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InputDebugging : ModuleRules
	{
		public InputDebugging(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ApplicationCore",
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"Slate",
					"SlateCore",
				}
			);
			
			if (Target.bBuildEditor)
			{
				PublicDependencyModuleNames.AddRange(
					new string[] {
						"UnrealEd"
					}
				);
			}
		}
	}
}
