// Copyright Epic Games, Inc. All Rights Reserved.

#include "IInputDebuggingInterface.h"
#include "Debug/DebugDrawService.h"
#include "Engine/Canvas.h"
#include "TouchInputVisualizer.h"
#include "InputDeviceDebugTools.h"
#include "Modules/ModuleManager.h"

class FInputDebuggingModule : public IInputDebuggingInterface
{
public:

	virtual void StartupModule() override
	{
		// register modular feature
		check(!IInputDebuggingInterface::IsAvailable()); // there can be only one
		IModularFeatures::Get().RegisterModularFeature(IInputDebuggingInterface::GetModularFeatureName(), this);

		// register debug rendering callback
		DrawOnCanvasDelegateHandle = UDebugDrawService::Register(TEXT("InputDebugVisualizer"), FDebugDrawDelegate::CreateRaw(this, &FInputDebuggingModule::OnDebugDraw));

		// initialize the touch input visualizer
#if SUPPORT_TOUCH_INPUT_DISPLAY
		TouchInputVisualizer = MakeShared<FTouchInputVisualizer>();
#endif//SUPPORT_TOUCH_INPUT_DISPLAY

#if SUPPORT_INPUT_DEVICE_DEBUGGING
		InputDeviceVisualizer = MakeShared<FInputDeviceDebugTools>();
#endif	// SUPPORT_INPUT_DEVICE_DEBUGGING
	}

	virtual void ShutdownModule() override
	{
		// shutdown the touch input visualizer
#if SUPPORT_TOUCH_INPUT_DISPLAY
		TouchInputVisualizer.Reset();
#endif//SUPPORT_TOUCH_INPUT_DISPLAY

#if SUPPORT_INPUT_DEVICE_DEBUGGING
		InputDeviceVisualizer.Reset();
#endif	// SUPPORT_INPUT_DEVICE_DEBUGGING

		// unregister debug rendering callback
		UDebugDrawService::Unregister(DrawOnCanvasDelegateHandle);

		// unregister modular feature
		IModularFeatures::Get().UnregisterModularFeature(IInputDebuggingInterface::GetModularFeatureName(), this);
	}


	void OnDebugDraw(UCanvas* Canvas, class APlayerController* PlayerController)
	{
		// debug render the touch input visualization.
#if SUPPORT_TOUCH_INPUT_DISPLAY
		TouchInputVisualizer->OnDebugDraw(Canvas);
#endif //SUPPORT_TOUCH_INPUT_DISPLAY
	}

private:

	FDelegateHandle DrawOnCanvasDelegateHandle;

#if SUPPORT_TOUCH_INPUT_DISPLAY
	TSharedPtr<FTouchInputVisualizer> TouchInputVisualizer;
#endif//SUPPORT_TOUCH_INPUT_DISPLAY

#if SUPPORT_INPUT_DEVICE_DEBUGGING
	TSharedPtr<FInputDeviceDebugTools> InputDeviceVisualizer;
#endif	// SUPPORT_INPUT_DEVICE_DEBUGGING
};

IMPLEMENT_MODULE(FInputDebuggingModule, InputDebugging);
