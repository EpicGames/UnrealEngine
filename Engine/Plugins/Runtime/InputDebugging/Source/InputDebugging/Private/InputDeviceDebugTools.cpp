// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputDeviceDebugTools.h"

#if SUPPORT_INPUT_DEVICE_DEBUGGING

#include "DrawDebugHelpers.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/HUD.h"
#include "GameFramework/InputDeviceProperties.h"

#if WITH_EDITOR
#include "Settings/LevelEditorPlaySettings.h"
#endif

FInputDeviceDebugTools::FInputDeviceDebugTools()
{
#if ENABLE_DRAW_DEBUG
	AHUD::OnShowDebugInfo.AddStatic(&FInputDeviceDebugTools::OnShowDebugInfo);
#endif
	AddConsoleCommands();

#if WITH_EDITOR
	if (ULevelEditorPlaySettings* Settings = GetMutableDefault<ULevelEditorPlaySettings>())
	{
		Settings->SimulatedDeviceMappingPolicyPostEditChange.AddLambda([]()
		{
			const ULevelEditorPlaySettings* EditorSettings = GetDefault<ULevelEditorPlaySettings>();
			// Notify the device mapper that we have changed and have it re-map input devices
			if (!EditorSettings->IsSimulateDeviceMappingPolicy())
			{
				// change it back to whatever the default project setting is
				if (const UInputPlatformSettings* Settings = UInputPlatformSettings::Get())
				{
					IPlatformInputDeviceMapper::Get().HandleInputDevicePolicyChanged(Settings->DeviceMappingPolicy);
				}
			}
			// If the policy is not invalid, then have the device mapper change to whatever it is now set to
			else
			{
				IPlatformInputDeviceMapper::Get().HandleInputDevicePolicyChanged(EditorSettings->GetSimulatedDeviceMappingPolicy());
			}
		});
	}
#endif	// WITH_EDITOR
}

FInputDeviceDebugTools::~FInputDeviceDebugTools()
{
	RemoveConsoleCommands();
}

void FInputDeviceDebugTools::AddConsoleCommands()
{
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("Input.ListAllHardwareDevices"),
		TEXT("Log all the platform's currently available FHardwareDeviceIdentifier"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateRaw(this, &FInputDeviceDebugTools::ListAllKnownHardwareDeviceIdentifier),
		ECVF_Cheat
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("Input.LogAllConnectedDevices"),
		TEXT("Log all the platform's currently connected devices from the device mapper"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateRaw(this, &FInputDeviceDebugTools::LogAllConnectedDevices),
		ECVF_Cheat
	));
}

void FInputDeviceDebugTools::RemoveConsoleCommands()
{
	// Unregister console commands
	for (IConsoleCommand* Command : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Command);
	}
}

void FInputDeviceDebugTools::OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	static const FName NAME_DeviceProperty("DeviceProperty");
	static const FName NAME_HardwareDevice("Devices");

	if (Canvas)
	{
		if (HUD->ShouldDisplayDebug(NAME_DeviceProperty))
		{
			OnShowDebugDeviceProperties(Canvas);
		}

		if (HUD->ShouldDisplayDebug(NAME_HardwareDevice))
		{
			OnShowDebugHardwareDevices(Canvas);
		}
	}
}

void FInputDeviceDebugTools::OnShowDebugDeviceProperties(UCanvas* Canvas)
{
	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;

	// Show title
	DisplayDebugManager.SetFont(GEngine->GetMediumFont());
	DisplayDebugManager.SetDrawColor(FColor::Orange);
	DisplayDebugManager.DrawString(TEXT("\n\nInput Device Properties"));

	const UInputDeviceSubsystem* System = UInputDeviceSubsystem::Get();
	if (!System)
	{
		DisplayDebugManager.SetDrawColor(FColor::Red);
		DisplayDebugManager.DrawString(TEXT("\t\tUInputDeviceSubsystem is not initalized!"));
		return;
	}

	if (System->ActiveProperties.IsEmpty())
	{
		DisplayDebugManager.SetDrawColor(FColor::White);
		DisplayDebugManager.DrawString(TEXT("\t \t No active device properties"));
		return;
	}

	// Sort the active properties by their owning Platform User
	static TMap<FPlatformUserId, TArray<const FActiveDeviceProperty*>> PlatformUserDebugStrings;
	{
		PlatformUserDebugStrings.Empty();

		for (const FActiveDeviceProperty& ActiveProperty : System->ActiveProperties)
		{
			PlatformUserDebugStrings.FindOrAdd(ActiveProperty.PlatformUser).Emplace(&ActiveProperty);
		}
	}	

	// Actually display the debug information	
	for (TPair<FPlatformUserId, TArray<const FActiveDeviceProperty*>> Pair : PlatformUserDebugStrings)
	{
		const FString PlatUserLabel = FString::Printf(TEXT("Platform User %d has %d active Device Properties"),
			Pair.Key.GetInternalId(),
			Pair.Value.Num());

		DisplayDebugManager.SetDrawColor(FColor::Yellow);
		DisplayDebugManager.DrawString(PlatUserLabel);

		for (const FActiveDeviceProperty* ActiveProperty: Pair.Value)
		{
			// Draw the name of the property
			DisplayDebugManager.SetDrawColor(FColor::White);
			DisplayDebugManager.DrawString(ActiveProperty->Property->GetFName().ToString());

			// Draw some information about the flags that have been set on the given property
			DisplayDebugManager.SetDrawColor(FColor::Silver);
			DisplayDebugManager.DrawString(FString::Printf(TEXT("\t Property Handle: %s"), *ActiveProperty->PropertyHandle.ToString()));
			DisplayDebugManager.DrawString(FString::Printf(TEXT("\t Playing on DeviceId: %d"), ActiveProperty->DeviceId.GetId()));
			DisplayDebugManager.DrawString(FString::Printf(TEXT("\t Flags: \n \t \t \t bLooping: %d \n \t \t \t bIgnoreTimeDilation: %d \n \t \t \t bPlayWhilePaused: %d \n \t \t \t bHasBeenAppliedAtLeastOnce: %d"),
				ActiveProperty->bLooping, ActiveProperty->bIgnoreTimeDilation, ActiveProperty->bPlayWhilePaused, ActiveProperty->bHasBeenAppliedAtLeastOnce));

			// Highlight if a property's evaluation has completed. If can go higher then the duration if
			// is set to not be removed after its evaluation is complete
			if (ActiveProperty->EvaluatedDuration > ActiveProperty->Property->GetDuration())
			{
				DisplayDebugManager.SetDrawColor(FColor::Green);
			}

			DisplayDebugManager.DrawString(FString::Printf(TEXT("\t Eval Time: %f"), ActiveProperty->EvaluatedDuration));			

			// TODO_BH: Pass in the canvas the the device property here to add an additional cool drawings
			// this will let use draw images, graphs, and a bunch of other cool visualizers
			//ActiveProperty.Property->DrawDebug(Canvas);
		}					
	}
}

void FInputDeviceDebugTools::OnShowDebugHardwareDevices(UCanvas* Canvas)
{
	const UInputDeviceSubsystem* System = UInputDeviceSubsystem::Get();
	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;

	DisplayDebugManager.SetFont(GEngine->GetMediumFont());

	if (!System)
	{
		DisplayDebugManager.SetDrawColor(FColor::Red);
		DisplayDebugManager.DrawString(TEXT("UInputDeviceSubsystem is not initalized!"));
		return;
	}	

	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
	
	// Title
	DisplayDebugManager.SetDrawColor(FColor::Orange);
	DisplayDebugManager.DrawString(TEXT("Input Devices"));

	TArray<FPlatformUserId> AllUsers;
	DeviceMapper.GetAllActiveUsers(OUT AllUsers);

	// Info relevant to all platform users
	const int32 NumUsers = DeviceMapper.GetAllActiveUsers(AllUsers);

	DisplayDebugManager.SetDrawColor(FColor::White);
	DisplayDebugManager.DrawString(FString::Printf(TEXT("Total Platform Users: %d"), NumUsers));

	for (const FPlatformUserId UserId : AllUsers)
	{
		TArray<FInputDeviceId> AllDevices;
		const int32 NumDevices = DeviceMapper.GetAllInputDevicesForUser(UserId, OUT AllDevices);

		// Show how many input devices each user has
		DisplayDebugManager.SetDrawColor(NumDevices > 0 ? FColor::Green : FColor::Orange);
		DisplayDebugManager.DrawString(FString::Printf(TEXT("Platform User '%d' has '%d' input devices"), UserId.GetInternalId(), NumDevices));

		for (const FInputDeviceId DeviceId : AllDevices)
		{
			static const float InputDeviceLabelXOffset = 10.0f;
			static const float InputDeviceInfoXOffset = InputDeviceLabelXOffset + 10.0f;

			// Print some information about the known hardware for this input device
			const FHardwareDeviceIdentifier Hardware = System->GetInputDeviceHardwareIdentifier(DeviceId);
			{
				const FString HardwareDeviceInfo = FString::Printf(TEXT("Input Device ID %d \t (%s::%s)"),
					DeviceId.GetId(),
					*Hardware.InputClassName.ToString(),
					*Hardware.HardwareDeviceIdentifier.ToString());			

				DisplayDebugManager.SetDrawColor(DeviceId.IsValid() ? FColor::White : FColor::Red);
				DisplayDebugManager.DrawString(HardwareDeviceInfo, InputDeviceLabelXOffset);
			}
			
			// Show info about how this input device is seen by Slate
			if (FSlateApplication::IsInitialized())
			{
				TOptional<int32> SlateUserId = FSlateApplication::Get().GetUserIndexForInputDevice(DeviceId);					
				if (SlateUserId.IsSet())
				{
					DisplayDebugManager.SetDrawColor(FColor::White);
					DisplayDebugManager.DrawString(FString::Printf(TEXT("Slate User Index: %d"), SlateUserId.GetValue()), InputDeviceInfoXOffset);
				}
				else
				{
					DisplayDebugManager.SetDrawColor(FColor::Red);
					DisplayDebugManager.DrawString(FString::Printf(TEXT("Slate User Index: INVALID")), InputDeviceInfoXOffset);
				}
			}

			// Display the connection status of the input device
			{
				const EInputDeviceConnectionState DeviceState = DeviceMapper.GetInputDeviceConnectionState(DeviceId);

				static const TMap<EInputDeviceConnectionState, FColor> ConnectionColorMap = 
				{
					{ EInputDeviceConnectionState::Invalid,			FColor::Red },
					{ EInputDeviceConnectionState::Unknown,			FColor::Orange },
					{ EInputDeviceConnectionState::Disconnected,	FColor::Silver },
					{ EInputDeviceConnectionState::Connected,		FColor::Green },
				};

				static const TMap<EInputDeviceConnectionState, FString> ConnectionLabelMap =
				{
					{ EInputDeviceConnectionState::Invalid,			TEXT("INVALID") },
					{ EInputDeviceConnectionState::Unknown,			TEXT("Unknown") },
					{ EInputDeviceConnectionState::Disconnected,	TEXT("Disconnected") },
					{ EInputDeviceConnectionState::Connected,		TEXT("Connected") },
				};

				if (const FColor* Color = ConnectionColorMap.Find(DeviceState))
				{
					DisplayDebugManager.SetDrawColor(*Color);
				}

				if (const FString* Label = ConnectionLabelMap.Find(DeviceState))
				{
					DisplayDebugManager.DrawString(*Label, InputDeviceInfoXOffset);
				}			
			}
		}		
	}
}

void FInputDeviceDebugTools::ListAllKnownHardwareDeviceIdentifier(const TArray<FString>& Args, UWorld* World)
{
	if (UInputPlatformSettings* System = UInputPlatformSettings::Get())
	{
		const TArray<FHardwareDeviceIdentifier>& Devices = System->GetHardwareDevices();
		for (const FHardwareDeviceIdentifier& Device : Devices)
		{
			UE_LOGF(LogInputDeviceProperties, Log, "%ls::%ls", *Device.InputClassName.ToString(), *Device.HardwareDeviceIdentifier.ToString());
		}
	}
}

void FInputDeviceDebugTools::LogAllConnectedDevices(const TArray<FString>& Args, UWorld* World)
{
	const UInputDeviceSubsystem* System = UInputDeviceSubsystem::Get();

	if (!System)
	{
		UE_LOGF(LogInput, Error, "[%s] 'Input.LogAllConnectedDevices' command failed! Unable to find a valid UInputDeviceSubsystem!", __func__);
		return;
	}

	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();

	TArray<FPlatformUserId> AllUsers;
	DeviceMapper.GetAllActiveUsers(OUT AllUsers);

	// Info relevant to all platform users
	const int32 NumUsers = DeviceMapper.GetAllActiveUsers(AllUsers);

	FString LogMessage = FString::Printf(TEXT("Total number of active users: %d\n"), NumUsers);
	
	{
		const EInputDeviceMappingPolicy MappingPolicy = IPlatformInputDeviceMapper::Get().GetCurrentDeviceMappingPolicy();
		
		auto PolicyToString = [](const EInputDeviceMappingPolicy Policy)-> FString
		{
			switch (Policy)
			{
			case EInputDeviceMappingPolicy::Invalid: return TEXT("Invalid");
			case EInputDeviceMappingPolicy::UseManagedPlatformLogin: return TEXT("UseManagedPlatformLogin");
			case EInputDeviceMappingPolicy::PrimaryUserSharesKeyboardAndFirstGamepad: return TEXT("PrimaryUserSharesKeyboardAndFirstGamepad");
			case EInputDeviceMappingPolicy::CreateUniquePlatformUserForEachDevice: return TEXT("CreateUniquePlatformUserForEachDevice");
			case EInputDeviceMappingPolicy::MapAllDevicesToPrimaryUser: return TEXT("MapAllDevicesToPrimaryUser");
			}

			return TEXT("Unknown EInputDeviceMappingPolicy");
		};
		
		LogMessage.Append(FString::Printf(TEXT("Input Device Mapping Policy: %s\n"), *PolicyToString(MappingPolicy)));
	}

	for (const FPlatformUserId UserId : AllUsers)
	{
		TArray<FInputDeviceId> AllDevices;
		const int32 NumDevices = DeviceMapper.GetAllInputDevicesForUser(UserId, OUT AllDevices);

		LogMessage.Append(FString::Printf(TEXT("Platform User: %d\t\t %d devices\n"), UserId.GetInternalId(), AllDevices.Num()));

		for (const FInputDeviceId DeviceId : AllDevices)
		{
			const FHardwareDeviceIdentifier DeviceData = System->GetInputDeviceHardwareIdentifier(DeviceId);

			LogMessage.Append(FString::Printf(TEXT("\tInput Device Id: %d    Hardware Info: %s\n"), DeviceId.GetId(), *DeviceData.ToString()));
		}
	}

	UE_LOGF(LogInput, Log, "[%s] Logging all connected input devices from 'Input.LogAllConnectedDevices' command... \n%ls", __func__, *LogMessage);
}

#endif	// SUPPORT_INPUT_DEVICE_DEBUGGING
