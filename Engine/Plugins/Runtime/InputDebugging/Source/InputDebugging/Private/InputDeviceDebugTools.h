// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h" // IWYU pragma: keep

// Allow you to enable this debug drawing in a build.cs if you want, but default to when we are not in shipping.
#if !defined(SUPPORT_INPUT_DEVICE_DEBUGGING)
#define SUPPORT_INPUT_DEVICE_DEBUGGING 		!UE_BUILD_SHIPPING
#endif

#if SUPPORT_INPUT_DEVICE_DEBUGGING

#include "Templates/SharedPointer.h"

class UCanvas;
struct IConsoleCommand;
class UWorld;
class AHUD;
class FDebugDisplayInfo;

/**  */
class FInputDeviceDebugTools : public TSharedFromThis<FInputDeviceDebugTools>
{
public:

	FInputDeviceDebugTools();
	~FInputDeviceDebugTools();
	
	// non-copyable, non-movable
	FInputDeviceDebugTools(const FInputDeviceDebugTools& Other) = delete;
	FInputDeviceDebugTools(FInputDeviceDebugTools&& Other) = delete;

private:

	void AddConsoleCommands();
	void RemoveConsoleCommands();

	static void OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

	static void OnShowDebugDeviceProperties(UCanvas* Canvas);

	static void OnShowDebugHardwareDevices(UCanvas* Canvas);

	/** Log all the platform's currently available FHardwareDeviceIdentifier  */
	void ListAllKnownHardwareDeviceIdentifier(const TArray<FString>& Args, UWorld* World);

	void LogAllConnectedDevices(const TArray<FString>& Args, UWorld* World);

	/** Console commands registered by this device debugger */
	TArray<IConsoleCommand*> ConsoleCommands;
};

#endif // SUPPORT_INPUT_DEVICE_DEBUGGING
