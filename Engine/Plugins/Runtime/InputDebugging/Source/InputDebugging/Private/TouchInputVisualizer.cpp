// Copyright Epic Games, Inc. All Rights Reserved.

#include "TouchInputVisualizer.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Misc/CoreDelegates.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SViewport.h"

#if SUPPORT_TOUCH_INPUT_DISPLAY


static int32 GInputEnableTouchDebugVisualizer = 0;
static FAutoConsoleVariableRef CVarInputEnableTouchDebugVisualizer(
	TEXT("Input.Debug.ShowTouches"),
	GInputEnableTouchDebugVisualizer,
	TEXT("Whether to show touch input on screen.")
);



FTouchInputVisualizer::FTouchInputVisualizer()
{
	if (!IsRunningCommandlet() && !IsRunningDedicatedServer())
	{
		// install input hook after the engine is initialized
		FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
		{
			FSlateApplication::Get().RegisterInputPreProcessor(AsShared());
		});

		// remove input hook when the engine is shutting down
		FCoreDelegates::OnEnginePreExit.AddLambda([this]()
		{
			FSlateApplication::Get().UnregisterInputPreProcessor(AsShared());
		});
	}
}


bool FTouchInputVisualizer::HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsTouchEvent())
	{
		DebugTouchPoints.Add(MouseEvent.GetPointerIndex(), FDebugTouchPoint( MouseEvent.GetScreenSpacePosition(), MouseEvent.GetTouchForce() ));
	}

	return false;
}

bool FTouchInputVisualizer::HandleMouseButtonDownEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsTouchEvent())
	{
		DebugTouchPoints.Add( MouseEvent.GetPointerIndex(), FDebugTouchPoint( MouseEvent.GetScreenSpacePosition(), MouseEvent.GetTouchForce() ));
	}

	return false;
}

bool FTouchInputVisualizer::HandleMouseButtonUpEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsTouchEvent())
	{
		DebugTouchPoints.Remove( MouseEvent.GetPointerIndex() );
	}

	return false;
}


void FTouchInputVisualizer::OnDebugDraw(UCanvas* Canvas)
{
	if (GInputEnableTouchDebugVisualizer)
	{
		// adjust canvas transform (it defaults to a 1080p canvas but the touch points are in absolute screen positions)
		const FVector Scale(Canvas->ClipX / Canvas->SizeX, Canvas->ClipY / Canvas->SizeY, 1.0);
		Canvas->Canvas->PushRelativeTransform(FScaleMatrix(Scale));

		// draw a circle for each active touch point (NB. can't use DrawDebugCanvas2DCircle because it's not available in Test)
		static const FColor Color = FColor::Yellow;
		static const int32 NumSides = 20;
		static const float LineThickness = 3.0f;
		static const float AngleDelta = TWO_PI / NumSides;

		const float MaxRadius = (Canvas->SizeX * 0.02);
		for (auto& Itr : DebugTouchPoints)
		{
			const FDebugTouchPoint& TouchPoint = Itr.Value;
			FVector2D Center = TouchPoint.Center;
#if WITH_EDITOR
			if (GEngine && GEngine->GameViewport)
			{
				if (const SViewport* GameViewportWidget = GEngine->GameViewport->GetGameViewportWidget().Get())
				{
					// subtract the viewport position to make sure the debug circle is drawn at the correct position even if the viewport origin is not in the top-left corner of the screen
					Center = TouchPoint.Center - GameViewportWidget->GetCachedGeometry().GetAbsolutePosition();
				}
			}
#endif
			const float Radius = MaxRadius * (1.0f + TouchPoint.Pressure);

			FVector2D LastVertex = Center + (FVector2D::UnitX() * Radius);
			for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
			{
				const FVector2D Vertex = Center + (FVector2D::UnitX() * FMath::Cos(AngleDelta * (SideIndex + 1)) + FVector2D::UnitY() * FMath::Sin(AngleDelta * (SideIndex + 1))) * Radius;
				FCanvasLineItem LineItem(LastVertex, Vertex);
				LineItem.LineThickness = LineThickness;
				LineItem.SetColor(Color);
				Canvas->DrawItem(LineItem);
				LastVertex = Vertex;
			}
		}

		// restore original canvas transform
		Canvas->Canvas->PopTransform();
	}
}


#endif //SUPPORT_TOUCH_INPUT_DISPLAY
