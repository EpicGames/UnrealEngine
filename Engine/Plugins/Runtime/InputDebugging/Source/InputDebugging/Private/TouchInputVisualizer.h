// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"

#if !defined(SUPPORT_TOUCH_INPUT_DISPLAY)
#define SUPPORT_TOUCH_INPUT_DISPLAY !UE_BUILD_SHIPPING
#endif

#if SUPPORT_TOUCH_INPUT_DISPLAY
#include "Framework/Application/IInputProcessor.h"

class FTouchInputVisualizer : public IInputProcessor, public TSharedFromThis<FTouchInputVisualizer>
{
public:
	FTouchInputVisualizer();
	virtual ~FTouchInputVisualizer() = default;

	void OnDebugDraw(class UCanvas* Canvas);

protected:
	// IInputProcessor interface
	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override {};
	virtual bool HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent) override;
	virtual bool HandleMouseButtonDownEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent) override;
	virtual bool HandleMouseButtonUpEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent) override;

	struct FDebugTouchPoint
	{
		explicit FDebugTouchPoint( const FVector2D& Center, float Pressure)
		{
			this->Center = Center;
			this->Pressure = Pressure;
		}

		FVector2D Center;
		float Pressure;
	};

	TMap<uint32, FDebugTouchPoint> DebugTouchPoints;

};
#endif //SUPPORT_TOUCH_INPUT_DISPLAY
