// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InputDebuggingEditor : ModuleRules
	{
		public InputDebuggingEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ApplicationCore",
					"Core",
					"Engine",
					"MainFrame",
					"Slate",
					"SlateCore",				 					
				}
			);
		}
	}
}