// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputDebuggingEditorModule.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameFramework/InputDeviceSubsystem.h"
#include "GameFramework/InputSettings.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Interfaces/IMainFrameModule.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"

#define LOCTEXT_NAMESPACE "InputDebuggingEditor"

/** Custom style set for the input debugger */
class FInputDebuggerSlateStyle final : public FSlateStyleSet
{
public:
	FInputDebuggerSlateStyle()
		: FSlateStyleSet("InputDebugger")
	{
		SetParentStyleName(FAppStyle::GetAppStyleSetName());

		SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		// Input Debugger icon for gamepad
		static const FVector2D Icon40x40 = FVector2D(40.0f, 40.0f);

		Set("InputDebugger.Gamepad", new IMAGE_BRUSH_SVG("Starship/AssetIcons/PlayerController_16", Icon40x40));
	}
};

void FInputDebuggingEditorModule::StartupModule()
{
	if (!FApp::HasProjectName())
	{
		return;
	}

	// Listen for when input devices change
	IPlatformInputDeviceMapper::Get().GetOnInputDeviceConnectionChange().AddRaw(this, &FInputDebuggingEditorModule::OnInputDeviceConnectionChange);

	// Make a new style set for an input debugger, which will register any custom icons for the types in this module
	StyleSet = MakeShared<FInputDebuggerSlateStyle>();
	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

void FInputDebuggingEditorModule::ShutdownModule()
{
	if (!FApp::HasProjectName())
	{
		return;
	}

	// Remove any input listeners
	IPlatformInputDeviceMapper::Get().GetOnInputDeviceConnectionChange().RemoveAll(this);

	// Unregister slate stylings
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
	}
}

const FText& FInputDebuggingEditorModule::ConnectionStateToText(const EInputDeviceConnectionState NewConnectionState)
{
	// "An input device..."
	static const FText InvalidLabel = LOCTEXT("InvalidStateLabel", "is in an invalid state");
	static const FText DisconnectedLabel = LOCTEXT("DisconnectedStateLabel", "has been disconnected");
	static const FText ConnectedLabel = LOCTEXT("ConnectedStateLabel", "has been connected.");
	static const FText UnknownLabel = LOCTEXT("UnknownStateLabel", "has an unknown state!");

	switch (NewConnectionState)
	{
		case EInputDeviceConnectionState::Invalid:
			return InvalidLabel;

		case EInputDeviceConnectionState::Disconnected:
			return DisconnectedLabel;

		case EInputDeviceConnectionState::Connected:
			return ConnectedLabel;

		case EInputDeviceConnectionState::Unknown:
		default:
			return UnknownLabel;
	}
}

void FInputDebuggingEditorModule::OnInputDeviceConnectionChange(EInputDeviceConnectionState NewConnectionState, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId)
{
	/** Utility functions for notifications */
	struct Local
	{

		static bool ShouldSuppressModal()
		{
			bool bSuppressNotification = false;
			GConfig->GetBool(TEXT("InputDebuggingEditor"), TEXT("SuppressInputDeviceConnectionChangeNotification"), bSuppressNotification, GEditorPerProjectIni);
			return bSuppressNotification;
		}

		static ECheckBoxState GetDontAskAgainCheckBoxState()
		{
			// Check the config for any preferences
			return ShouldSuppressModal() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		static void OnDontAskAgainCheckBoxStateChanged(ECheckBoxState NewState)
		{
			// If the user selects to not show this again, set that in the config so we know about it in between sessions
			const bool bSuppressNotification = (NewState == ECheckBoxState::Checked);
			GConfig->SetBool(TEXT("InputDebuggingEditor"), TEXT("SuppressInputDeviceConnectionChangeNotification"), bSuppressNotification, GEditorPerProjectIni);
		}
	};

	// If the user has specified to suppress this pop up, then just early out and exit
	if (Local::ShouldSuppressModal())
	{
		return;
	}

	// Send a slate notification that there was a new gamepad plugged in
	FFormatNamedArguments MainMessageArgs;
	MainMessageArgs.Add(TEXT("StateMessage"), ConnectionStateToText(NewConnectionState));
	const FText Message = FText::Format(LOCTEXT("InputDeviceConnectionChangeMessage", "An input device {StateMessage}"), MainMessageArgs);

	// Add details about what DeviceID and PlatformUserId it has been mapped to
	FFormatNamedArguments SubMessageArgs;
	SubMessageArgs.Add(TEXT("PlatformUser"), PlatformUserId.GetInternalId());
	SubMessageArgs.Add(TEXT("DeviceId"), InputDeviceId.GetId());
	
	const FHardwareDeviceIdentifier Hardware = UInputDeviceSubsystem::Get()->GetInputDeviceHardwareIdentifier(InputDeviceId);
	SubMessageArgs.Add(TEXT("HardwareDeviceID"), FText::FromString(Hardware.ToString()));
	
	const FText SubMessage = FText::Format(LOCTEXT("InputDeviceConnectionChangeSubMessage", "Platform User ID: {PlatformUser}\nDevice ID: {DeviceId}\nType: {HardwareDeviceID}"), SubMessageArgs);

	// Send out a slate notification that there was an input device connection change
	FNotificationInfo* Info = new FNotificationInfo(Message);
	Info->SubText = SubMessage;
	Info->ExpireDuration = 5.0f;
	Info->bFireAndForget = true;

	// Set the icon to a nifty gamepad
	if (StyleSet)
	{
		Info->Image = StyleSet->GetBrush(TEXT("InputDebugger.Gamepad"));
	}

	// Add a "Don't show this again" option
	Info->CheckBoxState = TAttribute<ECheckBoxState>::Create(&Local::GetDontAskAgainCheckBoxState);
	Info->CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(&Local::OnDontAskAgainCheckBoxStateChanged);
	Info->CheckBoxText = NSLOCTEXT("ModalDialogs", "DefaultCheckBoxMessage", "Don't show this again");

	FSlateNotificationManager::Get().QueueNotification(Info);
}

IMPLEMENT_MODULE(FInputDebuggingEditorModule, InputDebuggingEditor);

#undef LOCTEXT_NAMESPACE