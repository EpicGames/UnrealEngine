// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

class FSlateStyleSet;

class FInputDebuggingEditorModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:

	/**
	 * Returns an FText label for the given input device connection state
	 */
	static const FText& ConnectionStateToText(const EInputDeviceConnectionState NewConnectionState);

	/** Callback for when an input device has a connection state change (New gamepad plugged in, unplugged, etc) */
	void OnInputDeviceConnectionChange(EInputDeviceConnectionState NewConnectionState, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId);

	/** A custom slate style for input debugging related things */
	TSharedPtr<FSlateStyleSet> StyleSet;
};